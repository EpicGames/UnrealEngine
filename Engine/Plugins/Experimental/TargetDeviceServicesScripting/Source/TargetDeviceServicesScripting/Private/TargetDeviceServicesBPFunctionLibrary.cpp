// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetDeviceServicesBPFunctionLibrary.h"
#include "Modules/ModuleManager.h"
#include "ITargetDeviceServicesModule.h"
#include "ITargetDeviceProxyManager.h"
#include "ITargetDeviceService.h"
#include "ITargetDeviceProxy.h"

UTargetDeviceServicesBPFunctionLibrary::UTargetDeviceServicesBPFunctionLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

TMap<FString, FDeviceSnapshots> UTargetDeviceServicesBPFunctionLibrary::GetDeviceSnapshots()
{
	ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::GetModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");
	TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager = TargetDeviceServicesModule.GetDeviceProxyManager();
	TSharedRef<ITargetDeviceServiceManager> DeviceServiceManager = TargetDeviceServicesModule.GetDeviceServiceManager();

	TArray<ITargetDeviceServicePtr> DeviceServices;
	const int32 DeviceServicesCount = DeviceServiceManager->GetServices(DeviceServices);

	TMap<FString, FDeviceSnapshots> Snapshots;
	for (int32 CurrentDeviceServiceIndex = 0; CurrentDeviceServiceIndex < DeviceServicesCount; ++CurrentDeviceServiceIndex)
	{
		const ITargetDeviceServicePtr& DeviceService = DeviceServices[CurrentDeviceServiceIndex];

		if (DeviceService.IsValid())
		{
			// Fetching shared pointer to default device by passing into the ITargetDeviceService::GetDevice method Name_None
			ITargetDevicePtr DefaultDevice = DeviceService->GetDevice(NAME_None);

			if (DefaultDevice.IsValid())
			{
				const FString DefaultDeviceType = TargetDeviceTypes::ToString(DefaultDevice->GetDeviceType());
				FString DefaultDeviceId = DefaultDevice->GetId().ToString();
				TSharedPtr<ITargetDeviceProxy> DefaultDeviceProxy = 
					DeviceProxyManager->FindProxyDeviceForTargetDevice(DefaultDeviceId);

				FDeviceSnapshot DeviceSnapshot{
					// Fills the Name field
					DeviceService->GetDeviceName(),
					// Fills the HostName field
					DefaultDeviceProxy.IsValid() ? DefaultDeviceProxy->GetHostName() : DefaultDevice->GetName(),
					// Fills the DeviceType field
					DefaultDeviceType,
					// fills the ModelId field
					DefaultDevice->GetModelId(),
					// fills the DeviceConnectionType;
					DefaultDeviceProxy.IsValid() ? DefaultDeviceProxy->GetConnectionType() : TEXT(""),
					// fills the DeviceId field
					MoveTemp(DefaultDeviceId),
					// fills the OperatingSystem field
					DeviceService->GetDevicePlatformDisplayName(),
					// fills the IsConnected field
					DefaultDevice->IsConnected()
				};

				FDeviceSnapshots* DeviceSnapshotsOfCurrentDeviceTypePtr = Snapshots.Find(DefaultDeviceType);
				if (nullptr == DeviceSnapshotsOfCurrentDeviceTypePtr)
				{
					//Initializer list constructor call
					TArray<FDeviceSnapshot> Entries{MoveTemp(DeviceSnapshot)};
					Snapshots.Emplace(DefaultDeviceType, FDeviceSnapshots{MoveTemp(Entries)});
				}
				else
				{
					DeviceSnapshotsOfCurrentDeviceTypePtr->Entries.Emplace(MoveTemp(DeviceSnapshot));
				}
			}
		}
	}

	return MoveTemp(Snapshots);
}

