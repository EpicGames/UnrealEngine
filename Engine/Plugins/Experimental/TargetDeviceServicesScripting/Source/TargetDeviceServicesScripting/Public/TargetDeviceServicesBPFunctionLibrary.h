// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "TargetDeviceServicesBPFunctionLibrary.generated.h"

/**
* The struct is designed to store device information
*/
USTRUCT(BlueprintType)
struct FDeviceSnapshot
{
	GENERATED_BODY()

public:

	/** Stores device's name. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString Name;

	/** Stores device's hostname. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString HostName;

	/** Stores device's type. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString DeviceType;

	/** Stores device's model identifier. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString ModelId;

	/** Stores device's connection type. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString DeviceConnectionType;

	/** Stores device's identifier. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString DeviceId;
	
	/** Stores device's operating system name. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	FString OperatingSystem;
	
	/**
	* Stores device's flag that is used to detect whether device
	* is connected (true) or disconnected (false). 
	*/
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	bool IsConnected = false;
};

/**
* The struct is a container class that stores instances of FDeviceSnapshot instances.
* It is a helper class to be used in blueprints as a value of TMap container.
*/
USTRUCT(BlueprintType)
struct FDeviceSnapshots
{
	GENERATED_BODY()

public:

	/** Stores array of device snapshots. */
	UPROPERTY(BlueprintReadOnly, Category = "TargetDeviceServicesScripting")
	TArray<FDeviceSnapshot> Entries;
};

/**
* The class declares a set of functions to be exposed to blueprints.
*/
UCLASS(meta = (ScriptName = "TargetDeviceServices"))
class UTargetDeviceServicesBPFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()
	
public:
	/**
	* Fetches snapshots of devices that are available in the network.
	* 
	* @return A dictionary of devices' informational snapshots that are grouped by device type (device type string is used as a key).
	*/
	UFUNCTION(BlueprintCallable, Category = "TargetDeviceServicesScripting")
	static TARGETDEVICESERVICESSCRIPTING_API TMap<FString, FDeviceSnapshots> GetDeviceSnapshots();
};

