// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class TargetDeviceServicesScripting : ModuleRules
	{
		public TargetDeviceServicesScripting(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"Engine",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"TargetDeviceServices",
				});

			ShortName = "TDSScripting";
		}
	}
}
