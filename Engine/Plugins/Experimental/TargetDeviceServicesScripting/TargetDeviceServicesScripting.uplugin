{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "TargetDeviceServices scripting library",
	"Description": "Set of blueprint functions that enables working with TargetDeviceServices module via scripting",
	"Category": "Scripting",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "TargetDeviceServicesScripting",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}